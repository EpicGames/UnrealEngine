{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "2.0",
	"FriendlyName" : "Multicast Analytics Provider",
	"Description" : "Forwards analytics API calls to a list of analytics providers to log data to multiple services at once",
	"Category" : "Analytics",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://docs.unrealengine.com/latest/INT/Gameplay/Analytics/index.html",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AnalyticsMulticastEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "AnalyticsMulticast",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		}
	]
}