// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnalyticsMulticastEditor : ModuleRules
{
    public AnalyticsMulticastEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "Analytics",
                "AnalyticsVisualEditing",
				"Slate",
				"SlateCore",
				"Engine",
				"EditorFramework",
				"UnrealEd", // for Asset Editor Subsystem
				"PropertyEditor",
				"WorkspaceMenuStructure",				
				"EditorWidgets",
				"Projects",
				"DeveloperSettings"
			}
			);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
                "AssetTools"
            }
        );

	}
}
