// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnalyticsSettings.h"
#include "AnalyticsMulticastSettings.generated.h"

UCLASS()
class UAnalyticsMulticastSettings
	: public UAnalyticsSettingsBase
{
	GENERATED_UCLASS_BODY()

	/** The list of analytics providers to forward analytics events to */
	UPROPERTY(EditAnywhere, DisplayName="Release Providers", Category=Multicast, meta=(ConfigRestartRequired=true))
	TArray<FString> ReleaseMulticastProviders;

	/** The list of analytics providers to forward analytics events to */
	UPROPERTY(EditAnywhere, DisplayName="Debug Providers", Category=Multicast, meta=(ConfigRestartRequired=true))
	TArray<FString> DebugMulticastProviders;

	/** The list of analytics providers to forward analytics events to */
	UPROPERTY(EditAnywhere, DisplayName="Test Providers", Category=Multicast, meta=(ConfigRestartRequired=true))
	TArray<FString> TestMulticastProviders;

	/** The list of analytics providers to forward analytics events to */
	UPROPERTY(EditAnywhere, DisplayName="Development Providers", Category=Multicast, meta=(ConfigRestartRequired=true))
	TArray<FString> DevelopmentMulticastProviders;

// UAnalyticsSettingsBase interface
protected:
	/**
	 * Provides a mechanism to read the section based information into this UObject's properties
	 */
	virtual void ReadConfigSettings();
	/**
	 * Provides a mechanism to save this object's properties to the section based ini values
	 */
	virtual void WriteConfigSettings();

private:
	/** Helper to populate an array with the results of the config read */
	void BuildArrayFromString(const FString& List, TArray<FString>& Array);
	/** Helper to build the comma delimited string from the array */
	FString BuildStringFromArray(TArray<FString>& Array);
};
