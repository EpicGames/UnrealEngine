// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsMulticastEditor.h"

#include "AnalyticsMulticastSettings.h"
#include "Analytics.h"

IMPLEMENT_MODULE( FAnalyticsMulticastEditorModule, AnalyticsMulticastEditor );

#define LOCTEXT_NAMESPACE "AnalyticsMulticast"

void FAnalyticsMulticastEditorModule::StartupModule()
{
}

void FAnalyticsMulticastEditorModule::ShutdownModule()
{
}

UAnalyticsMulticastSettings::UAnalyticsMulticastSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SettingsDisplayName = LOCTEXT("SettingsDisplayName", "Multicast");
	SettingsTooltip = LOCTEXT("SettingsTooltip", "Configuration settings for the Multicast Analytics Provider");
}

void UAnalyticsMulticastSettings::ReadConfigSettings()
{
	FString ProviderList = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetReleaseIniSection(), TEXT("ProviderModuleNames"), true);
	BuildArrayFromString(ProviderList, ReleaseMulticastProviders);

	FString ProviderList2 = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetTestIniSection(), TEXT("ProviderModuleNames"), true);
	if (ProviderList2.Len() == 0)
	{
		ProviderList2 = ProviderList;
	}
	BuildArrayFromString(ProviderList2, TestMulticastProviders);

	FString ProviderList3 = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetDebugIniSection(), TEXT("ProviderModuleNames"), true);
	if (ProviderList3.Len() == 0)
	{
		ProviderList3 = ProviderList;
	}
	BuildArrayFromString(ProviderList3, DebugMulticastProviders);

	FString ProviderList4 = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetDevelopmentIniSection(), TEXT("ProviderModuleNames"), true);
	if (ProviderList4.Len() == 0)
	{
		ProviderList4 = ProviderList;
	}
	BuildArrayFromString(ProviderList4, DevelopmentMulticastProviders);
}

void UAnalyticsMulticastSettings::WriteConfigSettings()
{
	FString ProviderList = BuildStringFromArray(ReleaseMulticastProviders);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), GetReleaseIniSection(), TEXT("ProviderModuleNames"), ProviderList);

	ProviderList = BuildStringFromArray(TestMulticastProviders);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), GetTestIniSection(), TEXT("ProviderModuleNames"), ProviderList);

	ProviderList = BuildStringFromArray(DebugMulticastProviders);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), GetDebugIniSection(), TEXT("ProviderModuleNames"), ProviderList);

	ProviderList = BuildStringFromArray(DevelopmentMulticastProviders);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), GetDevelopmentIniSection(), TEXT("ProviderModuleNames"), ProviderList);
}

void UAnalyticsMulticastSettings::BuildArrayFromString(const FString& List, TArray<FString>& Array)
{
	Array.Empty();
	List.ParseIntoArray(Array, TEXT(","), true);
}

FString UAnalyticsMulticastSettings::BuildStringFromArray(TArray<FString>& Array)
{
	FString List;

	for (auto ProviderName : Array)
	{
		if (List.Len() > 0)
		{
			List += TEXT(",");
		}
		List += ProviderName;
	}
	return List;
}

#undef LOCTEXT_NAMESPACE
