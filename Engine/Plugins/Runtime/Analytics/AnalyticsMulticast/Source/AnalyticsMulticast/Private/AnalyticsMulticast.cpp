// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsMulticast.h"
#include "Interfaces/IAnalyticsProvider.h"

#include "Analytics.h"

IMPLEMENT_MODULE( FAnalyticsMulticast, AnalyticsMulticast );

class FAnalyticsProviderMulticast : public IAnalyticsProvider
{
	/** Singleton for analytics */
	static TSharedPtr<IAnalyticsProvider> Provider;
	FAnalyticsProviderMulticast(const FAnalyticsMulticast::Config& ConfigValues, const FAnalyticsProviderConfigurationDelegate& GetConfigValue);

public:
	static TSharedPtr<IAnalyticsProvider> Create(const FAnalyticsMulticast::Config& ConfigValues, const FAnalyticsProviderConfigurationDelegate& GetConfigValue)
	{
		if (!Provider.IsValid())
		{
			Provider = TSharedPtr<IAnalyticsProvider>(new FAnalyticsProviderMulticast(ConfigValues, GetConfigValue));
		}
		return Provider;
	}
	static void Destroy()
	{
		Provider.Reset();
	}

	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void EndSession() override;
	virtual void FlushEvents() override;

	virtual void SetUserID(const FString& InUserID) override;
	virtual FString GetUserID() const override;

	virtual FString GetSessionID() const override;
	virtual bool SetSessionID(const FString& InSessionID) override;

	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void RecordItemPurchase(const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity) override;
	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider) override;
	virtual void RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount) override;

	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)  override;
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override;
	virtual int32 GetDefaultEventAttributeCount() const  override;
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const  override;

	virtual ~FAnalyticsProviderMulticast();

	bool HasValidProviders() const { return Providers.Num() > 0; }

	virtual void SetBuildInfo(const FString& InBuildInfo) override;
	virtual void SetGender(const FString& InGender) override;
	virtual void SetLocation(const FString& InLocation) override;
	virtual void SetAge(const int32 InAge) override;

	virtual void RecordItemPurchase(const FString& ItemId, int ItemQuantity, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordError(const FString& Error, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordProgress(const FString& ProgressType, const FString& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordProgress(const FString& ProgressType, const TArray<FString>& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;

private:
	TArray<TSharedPtr<IAnalyticsProvider> > Providers;
	TArray<FString> ProviderModules;
	TArray<FAnalyticsEventAttribute> DefaultEventAttributes;
};

TSharedPtr<IAnalyticsProvider> FAnalyticsProviderMulticast::Provider;

void FAnalyticsMulticast::StartupModule()
{
}

void FAnalyticsMulticast::ShutdownModule()
{
	FAnalyticsProviderMulticast::Destroy();
}

TSharedPtr<IAnalyticsProvider> FAnalyticsMulticast::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	if (GetConfigValue.IsBound())
	{
		Config ConfigValues;
		ConfigValues.ProviderModuleNames = GetConfigValue.Execute(Config::GetKeyNameForProviderModuleNames(), true);
		if (ConfigValues.ProviderModuleNames.IsEmpty())
		{
			UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider delegate did not contain required parameter %ls", *Config::GetKeyNameForProviderModuleNames());
			return NULL;
		}
		return CreateAnalyticsProvider(ConfigValues, GetConfigValue);
	}
	else
	{
		UE_LOGF(LogAnalytics, Warning, "CreateAnalyticsProvider called with an unbound delegate");
	}
	return NULL;
}

TSharedPtr<IAnalyticsProvider> FAnalyticsMulticast::CreateAnalyticsProvider(const Config& ConfigValues, const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	return FAnalyticsProviderMulticast::Create(ConfigValues, GetConfigValue);
}

/**
 * Perform any initialization.
 */
FAnalyticsProviderMulticast::FAnalyticsProviderMulticast(const FAnalyticsMulticast::Config& ConfigValues, const FAnalyticsProviderConfigurationDelegate& GetConfigValue)
{
	UE_LOGF(LogAnalytics, Verbose, "Initializing Multicast Analytics provider");

	if (GetConfigValue.IsBound())
	{
		TArray<FString> ModuleNamesArray;
		ConfigValues.ProviderModuleNames.ParseIntoArray(ModuleNamesArray, TEXT(","), true);
		for (TArray<FString>::TConstIterator it(ModuleNamesArray);it;++it)
		{
			TSharedPtr<IAnalyticsProvider> NewProvider = FAnalytics::Get().CreateAnalyticsProvider(FName(**it), GetConfigValue);
			if (NewProvider.IsValid())
			{
				Providers.Add(NewProvider);
				ProviderModules.Add(*it);
			}
		}
	}
}

FAnalyticsProviderMulticast::~FAnalyticsProviderMulticast()
{
	UE_LOGF(LogAnalytics, Verbose, "Destroying Multicast Analytics provider");
}

bool FAnalyticsProviderMulticast::StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
{
	bool bResult = false;
	for (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it;++it)
	{
		bResult |= (*it)->StartSession(Attributes);
	}
	return bResult;
}

void FAnalyticsProviderMulticast::EndSession()
{
	for (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it;++it)
	{
		(*it)->EndSession();
	}
}

void FAnalyticsProviderMulticast::FlushEvents()
{
	for (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it;++it)
	{
		(*it)->FlushEvents();
	}
}

void FAnalyticsProviderMulticast::SetUserID( const FString& InUserID )
{
	for (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it;++it)
	{
		(*it)->SetUserID(InUserID);
	}
}

FString FAnalyticsProviderMulticast::GetUserID() const 
{
	if (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it)
	{
		return (*it)->GetUserID();
	}
	return FString();
}

FString FAnalyticsProviderMulticast::GetSessionID() const
{
	// combine all session IDs into Module@@Session##Module@@Session...
	FString Result;
	for (int i=0;i<Providers.Num();++i)
	{
		if (i>0) Result += "##";
		Result += ProviderModules[i] + "@@" + Providers[i]->GetSessionID();
	}
	return Result;
}

bool FAnalyticsProviderMulticast::SetSessionID(const FString& InSessionID)
{
	bool bResult = false;
	// parse out the format from GetSessionID and set the SessionID in each provider.
	TArray<FString> SessionIDs;
	// parse out the module/session pairs
	InSessionID.ParseIntoArray(SessionIDs, TEXT("##"), true);
	for (TArray<FString>::TConstIterator it(SessionIDs);it;++it)
	{
		FString ModuleName, SessionID;
		// split out the module name/sessionID pair
		if (it->Split(TEXT("@@"), &ModuleName, &SessionID))
		{
			// find the module name in the list of existing providers
			for (int i=0;i<Providers.Num();++i)
			{
				// only set the session ID if it is not empty.
				if (ProviderModules[i] == ModuleName && !SessionID.IsEmpty())
				{
					bResult |= Providers[i]->SetSessionID(SessionID);
				}
			}
		}
	}
	return bResult;
}


void FAnalyticsProviderMulticast::RecordEvent( const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes )
{
	for (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it;++it)
	{
		(*it)->RecordEvent(EventName, Attributes);
	}
}

void FAnalyticsProviderMulticast::RecordItemPurchase( const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity )
{
	for (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it;++it)
	{
		(*it)->RecordItemPurchase(ItemId, Currency, PerItemCost, ItemQuantity);
	}
}

void FAnalyticsProviderMulticast::RecordCurrencyPurchase( const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider )
{
	for (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it;++it)
	{
		(*it)->RecordCurrencyPurchase(GameCurrencyType, GameCurrencyAmount, RealCurrencyType, RealMoneyCost, PaymentProvider);
	}
}

void FAnalyticsProviderMulticast::RecordCurrencyGiven( const FString& GameCurrencyType, int GameCurrencyAmount )
{
	for (TArray<TSharedPtr<IAnalyticsProvider> >::TConstIterator it(Providers);it;++it)
	{
		(*it)->RecordCurrencyGiven(GameCurrencyType, GameCurrencyAmount);
	}
}

void FAnalyticsProviderMulticast::SetBuildInfo(const FString& InBuildInfo)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->SetBuildInfo(InBuildInfo);
	}
}

void FAnalyticsProviderMulticast::SetGender(const FString& InGender)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->SetGender(InGender);
	}
}

void FAnalyticsProviderMulticast::SetLocation(const FString& InLocation)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->SetLocation(InLocation);
	}
}

void FAnalyticsProviderMulticast::SetAge(const int32 InAge)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->SetAge(InAge);
	}
}

void FAnalyticsProviderMulticast::RecordItemPurchase(const FString& ItemId, int ItemQuantity, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->RecordItemPurchase(ItemId, ItemQuantity, EventAttrs);
	}
}

void FAnalyticsProviderMulticast::RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->RecordCurrencyPurchase(GameCurrencyType, GameCurrencyAmount, EventAttrs);
	}
}

void FAnalyticsProviderMulticast::RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->RecordCurrencyGiven(GameCurrencyType, GameCurrencyAmount, EventAttrs);
	}
}

void FAnalyticsProviderMulticast::RecordError(const FString& Error, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->RecordError(Error, EventAttrs);
	}
}

void FAnalyticsProviderMulticast::RecordProgress(const FString& ProgressType, const FString& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->RecordProgress(ProgressType, ProgressHierarchy, EventAttrs);
	}
}

void FAnalyticsProviderMulticast::RecordProgress(const FString& ProgressType, const TArray<FString>& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->RecordProgress(ProgressType, ProgressHierarchy, EventAttrs);
	}
}

void FAnalyticsProviderMulticast::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	DefaultEventAttributes = Attributes;

	for (auto CurrentProvider : Providers)
	{
		CurrentProvider->SetDefaultEventAttributes(CopyTemp(DefaultEventAttributes));
	}
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderMulticast::GetDefaultEventAttributesSafe() const
{
	return DefaultEventAttributes;
}

int32 FAnalyticsProviderMulticast::GetDefaultEventAttributeCount() const
{
	return DefaultEventAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderMulticast::GetDefaultEventAttribute(int AttributeIndex) const
{
	return DefaultEventAttributes[AttributeIndex];
}

