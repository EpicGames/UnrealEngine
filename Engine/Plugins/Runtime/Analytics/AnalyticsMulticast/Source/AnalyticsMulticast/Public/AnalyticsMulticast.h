// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IAnalyticsProviderModule.h"
#include "Modules/ModuleManager.h"

class IAnalyticsProvider;

/**
 * Exposes a multicast provider that multicasts analytics events to multiple providers.
 * Configured using a comma separated list of provider modules. Each module then uses
 * The supplied configuration delegate to configure itself.
 */
class FAnalyticsMulticast : public IAnalyticsProviderModule
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalyticsMulticast& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalyticsMulticast >( "AnalyticsMulticast" );
	}

	//--------------------------------------------------------------------------
	// Configuration functionality
	//--------------------------------------------------------------------------
public:
	/** 
	 * Defines required configuration values for multicast analytics provider. 
	 * Basically, you provide a list of provider modules that define the providers you want
	 * to multicast events to. Beyond that, each provider module created will use the 
	 * provided configuration delegate to configure itself, so that configuration delegate
	 * must be able to configure each specific provider as well (see CreateAnalyticsProvider function below).
	 */
	struct Config
	{
		/** Comma separated list of analytics provider modules */
		FString ProviderModuleNames;

		/** KeyName required for APIKey configuration. */
		static FString GetKeyNameForProviderModuleNames() { return TEXT("ProviderModuleNames"); }
	};

	//--------------------------------------------------------------------------
	// provider factory functions
	//--------------------------------------------------------------------------
public:
	/**
	 * IAnalyticsProviderModule interface.
	 * Creates the analytics provider given a configuration delegate.
	 * The keys required exactly match the field names in the Config object. 
	 * 
	 * When a particular provider module is loaded, it will create an instance and use the
	 * provided Configuration delegate to configure each provider.
	 */
	virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const;

	
	/** 
	 * Construct an analytics provider directly from a config object (and a delegate to provide configuration to each configured provider).
	 */
	virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const Config& ConfigValues, const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const;

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

